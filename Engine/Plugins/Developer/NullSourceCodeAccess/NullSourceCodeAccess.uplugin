{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Linux Compiler-only Integration",
	"Description" : "Allows access to c++ projects while only looking for clang++",
	"Category" : "Programming",
	"CreatedBy" : "Salamanderrake",
	"CreatedByURL" : "",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "NullSourceCodeAccess",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "PreDefault",
			"PlatformAllowList" :
			[
				"Linux"
			]
		}
	]
}