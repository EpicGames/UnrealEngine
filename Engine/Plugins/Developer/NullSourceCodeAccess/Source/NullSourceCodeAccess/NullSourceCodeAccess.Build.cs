// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NullSourceCodeAccess : ModuleRules
	{
		public NullSourceCodeAccess(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"DesktopPlatform",
					"SourceCodeAccess"
				}
			);

			ShortName = "NullSCA";
		}
	}
}
