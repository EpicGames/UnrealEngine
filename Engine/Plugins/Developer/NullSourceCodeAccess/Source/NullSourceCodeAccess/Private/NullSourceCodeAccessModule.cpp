// Copyright Epic Games, Inc. All Rights Reserved.

#include "NullSourceCodeAccessModule.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"

IMPLEMENT_MODULE( FNullSourceCodeAccessModule, NullSourceCodeAccess );

void FNullSourceCodeAccessModule::StartupModule()
{
	// Bind our source control provider to the editor
	IModularFeatures::Get().RegisterModularFeature(TEXT("SourceCodeAccessor"), &NullSourceCodeAccessor );
}

void FNullSourceCodeAccessModule::ShutdownModule()
{
	// unbind provider from editor
	IModularFeatures::Get().UnregisterModularFeature(TEXT("SourceCodeAccessor"), &NullSourceCodeAccessor);
}

FNullSourceCodeAccessor& FNullSourceCodeAccessModule::GetAccessor()
{
	return NullSourceCodeAccessor;
}
