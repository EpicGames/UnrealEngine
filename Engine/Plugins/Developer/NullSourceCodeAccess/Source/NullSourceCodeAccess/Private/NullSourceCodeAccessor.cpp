// Copyright Epic Games, Inc. All Rights Reserved.

#include "NullSourceCodeAccessor.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Paths.h"
#include "HAL/PlatformProcess.h"

#define LOCTEXT_NAMESPACE "NullSourceCodeAccessor"
bool FNullSourceCodeAccessor::CanAccessSourceCode() const
{
	// assume there is a bundled toolchain for installed builds and for source a toolchain should be around
	return true;
}

FName FNullSourceCodeAccessor::GetFName() const
{
	return FName("NullSourceCodeAccessor");
}

FText FNullSourceCodeAccessor::GetNameText() const 
{
	return LOCTEXT("NullDisplayName", "Null Source Code Access");
}

FText FNullSourceCodeAccessor::GetDescriptionText() const
{
	return LOCTEXT("NullDisplayDesc", "Create a c++ project without an IDE installed.");
}

bool FNullSourceCodeAccessor::OpenSolution()
{
	return true;
}

bool FNullSourceCodeAccessor::OpenSolutionAtPath(const FString& InSolutionPath)
{
	FString Path = FPaths::GetPath(InSolutionPath);
	FPlatformProcess::ExploreFolder(*Path);

	return true;
}

bool FNullSourceCodeAccessor::DoesSolutionExist() const
{
	return false;
}

bool FNullSourceCodeAccessor::OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber)
{
	return false;
}

bool FNullSourceCodeAccessor::OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths) 
{
	return false;
}

bool FNullSourceCodeAccessor::AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules)
{
	return false;
}

bool FNullSourceCodeAccessor::SaveAllOpenDocuments() const
{
	return false;
}

void FNullSourceCodeAccessor::Tick(const float DeltaTime) 
{

}

#undef LOCTEXT_NAMESPACE
